{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "NNERuntimeCoreML",
	"Description": "CoreML backed runtime for the Neural Network Engine (NNE).",
	"Category": "ML",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://dev.epicgames.com/community/learning/courses/e7w/unreal-engine-neural-network-engine-nne",
	"MarketplaceURL": "",
	"SupportURL": "https://forums.unrealengine.com/t/course-neural-network-engine-nne/1162628",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "NNERuntimeCoreMLEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NNERuntimeCoreML",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NNERuntimeCoreMLUtils",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default"
		}
	]
}