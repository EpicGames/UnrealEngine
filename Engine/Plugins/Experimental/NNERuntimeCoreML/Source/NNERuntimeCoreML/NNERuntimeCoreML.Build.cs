// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NNERuntimeCoreML : ModuleRules
{
	public NNERuntimeCoreML( ReadOnlyTargetRules Target ) : base( Target )
	{
		bRequiresPlatformSDK = true;
		
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"NNE",
			"NNERuntimeCoreMLUtils",
			"Projects"
		});

		if (Target.Platform == UnrealTargetPlatform.Mac || Target.Type == TargetType.Editor)
		{
			PublicDefinitions.Add("WITH_NNE_RUNTIME_COREML");
		}
		
		if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicFrameworks.AddRange(new string[] { "CoreML" });
		}
	}
}
