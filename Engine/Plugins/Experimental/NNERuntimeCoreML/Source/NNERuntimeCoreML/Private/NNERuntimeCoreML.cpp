// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef WITH_NNE_RUNTIME_COREML

#include "NNERuntimeCoreML.h"

#include "CoreMinimal.h"
#include "NNE.h"
#include "NNEModelData.h"
#include "NNERuntimeCoreMLLog.h"
#include "NNERuntimeCoreMLModel.h"
#include "NNERuntimeCoreMLModelData.h"
#include "NNERuntimeCoreMLNPUHelper.h"

FGuid UNNERuntimeCoreML::GUID = FGuid((int32)'C', (int32)'O', (int32)'M', (int32)'L');
int32 UNNERuntimeCoreML::Version = 0x00000002;

namespace UE::NNERuntimeCoreML::Private::Details
{
	template <class CanCreateModelStatus> CanCreateModelStatus CanCreateModel(const TObjectPtr<UNNEModelData> ModelData, const FString& RuntimeName)
	{
		check(ModelData != nullptr);

		const TSharedPtr<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(RuntimeName);
		if (!SharedData.IsValid())
		{
			return CanCreateModelStatus::Fail;
		}

		FModelData CoreMLModelData;
		if (!CoreMLModelData.Load(SharedData->GetView()) || !CoreMLModelData.CheckGUIDAndVersion(UNNERuntimeCoreML::GUID, UNNERuntimeCoreML::Version))
		{
			return CanCreateModelStatus::Fail;
		}

		return CanCreateModelStatus::Ok;
	}
} // UE::NNERuntimeCoreML::Private::Details

UNNERuntimeCoreML::ECanCreateModelDataStatus UNNERuntimeCoreML::CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
	if (FileData.IsEmpty())
	{
		return ECanCreateModelDataStatus::Fail;
	}

	const bool bIsMLModel = FileType.Compare("mlmodel", ESearchCase::IgnoreCase) == 0;
	const bool bIsMLPackage = FileType.Compare("mlpackage", ESearchCase::IgnoreCase) == 0;
	if (!bIsMLModel && !bIsMLPackage)
	{
		return ECanCreateModelDataStatus::FailFileIdNotSupported;
	}

	return ECanCreateModelDataStatus::Ok;
}

TSharedPtr<UE::NNE::FSharedModelData> UNNERuntimeCoreML::CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform)
{
	using namespace UE::NNERuntimeCoreML;

	if (CanCreateModelData(FileType, FileData, AdditionalFileData, FileId, TargetPlatform) != ECanCreateModelDataStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Cannot create the model data with id %ls (Filetype: %ls)", *FileId.ToString(EGuidFormats::Digits).ToLower(), *FileType);
		return {};
	}
	//Note: here the model should be optimized and the related mlmodelc binary blob saved instead of source model
	
	const bool bIsMLModel = FileType.Compare("mlmodel", ESearchCase::IgnoreCase) == 0;
	
	FModelData ModelData
	{
		.GUID = UNNERuntimeCoreML::GUID,
		.Version = UNNERuntimeCoreML::Version,
		.ModelType = bIsMLModel ? EModelType::MLModel : EModelType::MLPackage,
		.FileDataView = FileData
	};
	
	TArray64<uint8> Result;
	if (!ModelData.Store(Result))
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Model data serialization failed");
		return {};
	}

	return MakeShared<UE::NNE::FSharedModelData>(MakeSharedBufferFromArray(MoveTemp(Result)), 0);
}

FString UNNERuntimeCoreML::GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const
{
	return FileId.ToString(EGuidFormats::Digits) + "-" + UNNERuntimeCoreML::GUID.ToString(EGuidFormats::Digits) + "-" + FString::FromInt(UNNERuntimeCoreML::Version);
}

FString UNNERuntimeCoreML::GetRuntimeName() const
{
	return TEXT("NNERuntimeCoreML");
}

UNNERuntimeCoreMLCpuGpu::ECanCreateModelCPUStatus UNNERuntimeCoreMLCpuGpu::CanCreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) const
{
	return UE::NNERuntimeCoreML::Private::Details::CanCreateModel<UNNERuntimeCoreMLCpuGpu::ECanCreateModelCPUStatus>(ModelData, GetRuntimeName());
}

UNNERuntimeCoreMLCpuGpu::ECanCreateModelGPUStatus UNNERuntimeCoreMLCpuGpu::CanCreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) const
{
	return UE::NNERuntimeCoreML::Private::Details::CanCreateModel<UNNERuntimeCoreMLCpuGpu::ECanCreateModelGPUStatus>(ModelData, GetRuntimeName());
}

UNNERuntimeCoreMLCpuGpuNpu::ECanCreateModelNPUStatus UNNERuntimeCoreMLCpuGpuNpu::CanCreateModelNPU(const TObjectPtr<UNNEModelData> ModelData) const
{
	if (!UE::NNERuntimeCoreML::Private::IsNPUAvailable())
		return UNNERuntimeCoreMLCpuGpuNpu::ECanCreateModelNPUStatus::Fail;

	return UE::NNERuntimeCoreML::Private::Details::CanCreateModel<UNNERuntimeCoreMLCpuGpuNpu::ECanCreateModelNPUStatus>(ModelData, GetRuntimeName());
}

TSharedPtr<UE::NNE::IModelCPU> UNNERuntimeCoreMLCpuGpu::CreateModelCPU(const TObjectPtr<UNNEModelData> ModelData)
{
	check(ModelData != nullptr);
	if (CanCreateModelCPU(ModelData) != ECanCreateModelCPUStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Cannot create a CPU model from the model data with id %ls", *ModelData->GetFileId().ToString(EGuidFormats::Digits));
		return TSharedPtr<UE::NNE::IModelCPU>();
	}

	const TSharedRef<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName()).ToSharedRef();

	return MakeShared<UE::NNERuntimeCoreML::Private::FModelCoreMLCpu>(SharedData);
}

TSharedPtr<UE::NNE::IModelGPU> UNNERuntimeCoreMLCpuGpu::CreateModelGPU(const TObjectPtr<UNNEModelData> ModelData)
{
	check(ModelData != nullptr);
	if (CanCreateModelGPU(ModelData) != ECanCreateModelGPUStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Cannot create a GPU model from the model data with id %ls", *ModelData->GetFileId().ToString(EGuidFormats::Digits));
		return TSharedPtr<UE::NNE::IModelGPU>();
	}

	const TSharedRef<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName()).ToSharedRef();

	return MakeShared<UE::NNERuntimeCoreML::Private::FModelCoreMLGpu>(SharedData);
}

TSharedPtr<UE::NNE::IModelNPU> UNNERuntimeCoreMLCpuGpuNpu::CreateModelNPU(const TObjectPtr<UNNEModelData> ModelData)
{
	check(ModelData != nullptr);
	if (CanCreateModelNPU(ModelData) != ECanCreateModelNPUStatus::Ok)
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Cannot create a NPU model from the model data with id %ls", *ModelData->GetFileId().ToString(EGuidFormats::Digits));
		return TSharedPtr<UE::NNE::IModelNPU>();
	}

	const TSharedRef<UE::NNE::FSharedModelData> SharedData = ModelData->GetModelData(GetRuntimeName()).ToSharedRef();

	return MakeShared<UE::NNERuntimeCoreML::Private::FModelCoreMLNpu>(SharedData);
}

#endif // WITH_NNE_RUNTIME_COREML
