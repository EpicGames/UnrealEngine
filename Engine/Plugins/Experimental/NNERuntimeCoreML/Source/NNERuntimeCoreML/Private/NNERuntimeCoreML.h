// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Misc/Guid.h"
#include "NNERuntime.h"
#include "NNERuntimeCPU.h"
#include "NNERuntimeGPU.h"
#include "NNERuntimeNPU.h"
#include "UObject/Object.h"

#include "NNERuntimeCoreML.generated.h"

UCLASS()
class UNNERuntimeCoreML : public UObject, public INNERuntime
{
	GENERATED_BODY()

public:
	virtual ~UNNERuntimeCoreML() = default;

#ifdef WITH_NNE_RUNTIME_COREML
	static FGuid GUID;
	static int32 Version;

	//~ Begin INNERuntime Interface
	virtual FString GetRuntimeName() const override;
	virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;
	virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) override;
	virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override;
	//~ End INNERuntime Interface
#else
	//~ Begin INNERuntime Interface
	virtual FString GetRuntimeName() const override { return ""; };
	virtual ECanCreateModelDataStatus CanCreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override { return ECanCreateModelDataStatus::Fail; };
	virtual TSharedPtr<UE::NNE::FSharedModelData> CreateModelData(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) override { return TSharedPtr<UE::NNE::FSharedModelData>(); };
	virtual FString GetModelDataIdentifier(const FString& FileType, TConstArrayView64<uint8> FileData, const TMap<FString, TConstArrayView64<uint8>>& AdditionalFileData, const FGuid& FileId, const ITargetPlatform* TargetPlatform) const override { return ""; };
	//~ End INNERuntime Interface
#endif // WITH_NNE_RUNTIME_COREML
};

UCLASS()
class UNNERuntimeCoreMLCpuGpu : public UNNERuntimeCoreML, public INNERuntimeCPU, public INNERuntimeGPU
{
	GENERATED_BODY()

public:
	virtual ~UNNERuntimeCoreMLCpuGpu() = default;

#ifdef WITH_NNE_RUNTIME_COREML
	//~ Begin INNERuntimeCPU Interface
	virtual ECanCreateModelCPUStatus CanCreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) const override;
	virtual TSharedPtr<UE::NNE::IModelCPU> CreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) override;
	//~ End INNERuntimeCPU Interface

	//~ Begin INNERuntimeGPU Interface
	virtual ECanCreateModelGPUStatus CanCreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) const override;
	virtual TSharedPtr<UE::NNE::IModelGPU> CreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) override;
	//~ End INNERuntimeGPU Interface
#else
	//~ Begin INNERuntimeCPU Interface
	virtual ECanCreateModelCPUStatus CanCreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) const override { return ECanCreateModelCPUStatus::Fail; };
	virtual TSharedPtr<UE::NNE::IModelCPU> CreateModelCPU(const TObjectPtr<UNNEModelData> ModelData) override { return TSharedPtr<UE::NNE::IModelCPU>(); };
	//~ End INNERuntimeCPU Interface

	//~ Begin INNERuntimeGPU Interface
	virtual ECanCreateModelGPUStatus CanCreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) const override { return ECanCreateModelGPUStatus::Fail; };
	virtual TSharedPtr<UE::NNE::IModelGPU> CreateModelGPU(const TObjectPtr<UNNEModelData> ModelData) override { return TSharedPtr<UE::NNE::IModelGPU>(); };
	//~ End INNERuntimeGPU Interface
#endif // WITH_NNE_RUNTIME_COREML
};

UCLASS()
class UNNERuntimeCoreMLCpuGpuNpu : public UNNERuntimeCoreMLCpuGpu, public INNERuntimeNPU
{
	GENERATED_BODY()

public:
	virtual ~UNNERuntimeCoreMLCpuGpuNpu() = default;

#ifdef WITH_NNE_RUNTIME_COREML
	//~ Begin INNERuntimeNPU Interface
	virtual ECanCreateModelNPUStatus CanCreateModelNPU(const TObjectPtr<UNNEModelData> ModelData) const override;
	virtual TSharedPtr<UE::NNE::IModelNPU> CreateModelNPU(const TObjectPtr<UNNEModelData> ModelData) override;
	//~ End INNERuntimeNPU Interface
#else
	//~ Begin INNERuntimeNPU Interface
	virtual ECanCreateModelNPUStatus CanCreateModelNPU(const TObjectPtr<UNNEModelData> ModelData) const override { return ECanCreateModelNPUStatus::Fail; };
	virtual TSharedPtr<UE::NNE::IModelNPU> CreateModelNPU(const TObjectPtr<UNNEModelData> ModelData) override { return TSharedPtr<UE::NNE::IModelNPU>(); };
	//~ End INNERuntimeNPU Interface
#endif // WITH_NNE_RUNTIME_COREML
};
