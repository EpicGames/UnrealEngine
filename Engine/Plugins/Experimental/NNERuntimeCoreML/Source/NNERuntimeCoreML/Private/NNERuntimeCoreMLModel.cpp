// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef WITH_NNE_RUNTIME_COREML

#include "NNERuntimeCoreMLModel.h"

#include "CoreMinimal.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformMisc.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "NNEModelData.h"
#include "NNERuntimeCoreML.h"
#include "NNERuntimeCoreMLLog.h"
#include "NNERuntimeCoreMLModelData.h"
#include "NNERuntimeCoreMLUtils.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#if defined(__APPLE__)
#import <Foundation/Foundation.h>
#import <CoreML/CoreML.h>

#include "Apple/ScopeAutoreleasePool.h"
#include "Apple/AppleStringUtils.h"

@interface CoreMLInputWrapper : NSObject<MLFeatureProvider>
@property (readwrite, nonatomic, retain) NSMutableDictionary<NSString*, MLFeatureValue*>* featureByNames;
@end

@implementation CoreMLInputWrapper
- (NSSet<NSString*>*) featureNames {
	NSSet* set = [NSSet setWithArray:_featureByNames.allKeys];
	return set;
}
- (nullable MLFeatureValue*) featureValueForName: (NSString*) featureName {
	MLFeatureValue* value = _featureByNames[featureName];
	return value;
}
@end

#endif // defined(__APPLE__)

namespace UE::NNERuntimeCoreML::Private
{

namespace Detail
{
#if defined(__APPLE__)

	template< typename T > 
	struct InstanceTypeTrait{ 
	};

	template<> 
	struct InstanceTypeTrait<UE::NNE::IModelInstanceCPU>{ 
		static MLComputeUnits GetComputeUnits() { return MLComputeUnitsCPUOnly; }
	};

	template<> 
	struct InstanceTypeTrait<UE::NNE::IModelInstanceGPU>{ 
		static MLComputeUnits GetComputeUnits() { return MLComputeUnitsCPUAndGPU; }
	};

	template<> 
	struct InstanceTypeTrait<UE::NNE::IModelInstanceNPU>{ 
		static MLComputeUnits GetComputeUnits() { return MLComputeUnitsCPUAndNeuralEngine; }
	};

	FString GenerateUniqueTempDirPath(const FString& BasePath)
	{
		FString UniqueDirPath;
		do
		{
			UniqueDirPath = FPaths::Combine(BasePath, *FString::Printf(TEXT("CoreMLModel_%s"), *FGuid::NewGuid().ToString()));
		} while (IFileManager::Get().DirectoryExists(*UniqueDirPath));

		return UniqueDirPath;
	}

	void FillNSArrayShapeAndStrideFromNNEShape(const NNE::FTensorShape& TensorShape, NSMutableArray<NSNumber*>* Shape, NSMutableArray<NSNumber*>* Strides)
	{
		TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> StridesData;
		int32 OngoingStride = 1;
		const int32 Rank = TensorShape.Rank();
		
		check(Rank > 0);
		StridesData.SetNum(Rank);
		for (int32 i = Rank-1; i >= 0 ; --i)
		{
			int32 DimSize = static_cast<int32>(TensorShape.GetData()[i]);
			
			StridesData[i] = OngoingStride;
			OngoingStride *= DimSize;
		}
		check(OngoingStride == TensorShape.Volume());
		
		[Shape removeAllObjects];
		[Strides removeAllObjects];
		for (int32 i = 0; i < Rank ; ++i)
		{
			int32 DimSize = static_cast<int32>(TensorShape.GetData()[i]);
			int32 DimStride = StridesData[i];
			
			[Shape addObject:[NSNumber numberWithInt:DimSize]];
			[Strides addObject:[NSNumber numberWithInt:DimStride]];
		}
	}

	ENNETensorDataType GetTypeFromMultiArrayDataType(MLMultiArrayDataType Type)
	{
		switch(Type)
		{
			case MLMultiArrayDataTypeInt32:		return ENNETensorDataType::Int32;
			case MLMultiArrayDataTypeFloat16:	return ENNETensorDataType::Half;
			case MLMultiArrayDataTypeFloat32:	return ENNETensorDataType::Float;
			case MLMultiArrayDataTypeFloat64:	return ENNETensorDataType::Double;
			default:
				return ENNETensorDataType::None;
		}
	}

	MLMultiArrayDataType GetMultiArrayDataTypeFromType(ENNETensorDataType Type)
	{
		switch(Type)
		{
			case ENNETensorDataType::Int32:		return MLMultiArrayDataTypeInt32;
			case ENNETensorDataType::Half:		return MLMultiArrayDataTypeFloat16;
			case ENNETensorDataType::Float:		return MLMultiArrayDataTypeFloat32;
			case ENNETensorDataType::Double:	return MLMultiArrayDataTypeFloat64;
			default:
				unimplemented();
				return MLMultiArrayDataTypeFloat32;
		}
	}

	template<typename T> TArray<T, TInlineAllocator<NNE::FTensorShape::MaxRank>> GetShapeDataFromNSArray(const NSArray<NSNumber*>* Shape)
	{
		TArray<T, TInlineAllocator<NNE::FTensorShape::MaxRank>> Data;
		for(NSNumber* Dim in Shape)
		{
			Data.Emplace(static_cast<T>(Dim.intValue));
		}
		return Data;
	}

	bool GetTensorDescAndFeatureNamesFromMLDescription(TArray<NNE::FTensorDesc>& TensorDescs,
										TArray<FString>& FeatureNames,
										const NSDictionary<NSString* ,MLFeatureDescription *>* FeatureDictionary, 
										const bool AreUndefinedShapesAnError, 
										const FString& Context)
	{
		TensorDescs.Reset();
		FeatureNames.Reset();
		bool AllTensorDescsValid = true;
		
		if (FeatureDictionary == nullptr)
		{
			UE_LOGF(LogNNERuntimeCoreML, Error, "Invalid feature dictionary for model %ls. Was nullptr expecting a valid dictionary.", *Context);
			return false;
		}

		for(NSString* Key in FeatureDictionary)
		{
			MLFeatureDescription* FeatureDescription = FeatureDictionary[Key];
			const FString FeatureName = FeatureDescription.name;
			
			if (FeatureDescription.optional)
			{
				UE_LOGF(LogNNERuntimeCoreML, Error, "Optional feature are not supported but %ls feature %ls is optional.", *Context, *FeatureName);
				TensorDescs.Reset();
				FeatureNames.Reset();
				return false;
			}
			if (FeatureDescription.type != MLFeatureTypeMultiArray)
			{
				UE_LOGF(LogNNERuntimeCoreML, Error, "Plugin is experimental: Only MultiArray are supported at the moment but %ls feature %ls is not", *Context, *FeatureName);
				TensorDescs.Reset();
				FeatureNames.Reset();
				return false;
			}
			
			if (FeatureDescription.multiArrayConstraint.shape.count != 0)
			{
				const TArray<int32, TInlineAllocator<NNE::FTensorShape::MaxRank>> SymbolicShapeData = GetShapeDataFromNSArray<int32>(FeatureDescription.multiArrayConstraint.shape);
				const ENNETensorDataType DataType = GetTypeFromMultiArrayDataType(FeatureDescription.multiArrayConstraint.dataType);
				const NNE::FSymbolicTensorShape SymbolicShape = NNE::FSymbolicTensorShape::Make(SymbolicShapeData);
				const NNE::FTensorDesc SymbolicTensor = NNE::FTensorDesc::Make(FeatureName, SymbolicShape, DataType);
				
				TensorDescs.Emplace(SymbolicTensor);
			}
			else // undefined MultiArray shape
			{
				if (AreUndefinedShapesAnError)
				{
					UE_LOGF(LogNNERuntimeCoreML, Error, "MultiArray features need to define a shape but %ls feature %ls does not", *Context, *FeatureName);
					TensorDescs.Reset();
					FeatureNames.Reset();
					return false;
				}
				else
				{
					AllTensorDescsValid = false;
				}
			}
			
			FeatureNames.Emplace(FeatureName);
		}
		
		if (!AllTensorDescsValid)
		{
			//if some CoreML features could not be expressed as TensorDescs we dont expose any of the metadata (or indices won't match).
			//however we still have registered there names so we will be able to run inference.
			TensorDescs.Reset();
		}
		
		return true;
	}
#endif // defined(__APPLE__)
} // namespace Detail

template <class InstanceType>
FModelInstanceCoreMLBase<InstanceType>::~FModelInstanceCoreMLBase()
{
#if defined(__APPLE__)
	[CoreMLModelInstance release];
#endif // defined(__APPLE__)
}

template <class InstanceType>
TConstArrayView<NNE::FTensorDesc> FModelInstanceCoreMLBase<InstanceType>::GetInputTensorDescs() const
{
	return InputSymbolicTensors;
}

template <class InstanceType>
TConstArrayView<NNE::FTensorDesc> FModelInstanceCoreMLBase<InstanceType>::GetOutputTensorDescs() const
{
	return OutputSymbolicTensors;
}

template <class InstanceType>
TConstArrayView<NNE::FTensorShape> FModelInstanceCoreMLBase<InstanceType>::GetInputTensorShapes() const
{
	return InputTensorShapes;
}

template <class InstanceType>
TConstArrayView<NNE::FTensorShape> FModelInstanceCoreMLBase<InstanceType>::GetOutputTensorShapes() const
{
	return OutputTensorShapes;
}

template <class InstanceType>
FModelInstanceCoreMLCpu::ESetInputTensorShapesStatus FModelInstanceCoreMLBase<InstanceType>::SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes)
{
	InputTensorShapes.Reset(InInputShapes.Num());
	OutputTensorShapes.Reset();

	if (InInputShapes.Num() != InputSymbolicTensors.Num())
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Number of input shapes does not match number of input tensors");
		return FModelInstanceCoreMLCpu::ESetInputTensorShapesStatus::Fail;
	}

	for (int32 i = 0; i < InInputShapes.Num(); ++i)
	{
		const NNE::FTensorDesc SymbolicDesc = InputSymbolicTensors[i];
		if (!InInputShapes[i].IsCompatibleWith(SymbolicDesc.GetShape()))
		{
			UE_LOGF(LogNNERuntimeCoreML, Error, "Input shape does not match input tensor %ls of index %d", *SymbolicDesc.GetName(), i);
			return FModelInstanceCoreMLCpu::ESetInputTensorShapesStatus::Fail;
		}
	}
		
	InputTensorShapes = InInputShapes;

	// Setup output shapes only if all model symbolic output shapes are concretes, otherwise it will be set during inference.
	for (NNE::FTensorDesc SymbolicTensorDesc : OutputSymbolicTensors)
	{
		if (SymbolicTensorDesc.GetShape().IsConcrete())
		{
			NNE::FTensorShape TensorShape = NNE::FTensorShape::MakeFromSymbolic(SymbolicTensorDesc.GetShape());
			OutputTensorShapes.Emplace(TensorShape);
		}
	}
	if (OutputTensorShapes.Num() != OutputSymbolicTensors.Num())
	{
		OutputTensorShapes.Reset();
	}

	return FModelInstanceCoreMLCpu::ESetInputTensorShapesStatus::Ok;
}

template <class InstanceType>
FModelInstanceCoreMLCpu::ERunSyncStatus FModelInstanceCoreMLBase<InstanceType>::RunSync(TConstArrayView<NNE::FTensorBindingCPU> InInputBindings, TConstArrayView<NNE::FTensorBindingCPU> InOutputBindings)
{
#if defined(__APPLE__)
	SCOPED_AUTORELEASE_POOL;
	
	if (InputTensorShapes.IsEmpty())
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Input shapes are not set, please call SetInputTensorShapes.");
		return FModelInstanceCoreMLCpu::ERunSyncStatus::Fail;
	}

	check(InputSymbolicTensors.Num() == InputTensorShapes.Num());

	if (InInputBindings.Num() != InputTensorShapes.Num())
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Input bindings need to match input tensor descriptor count (got %d, expected %d).", InInputBindings.Num(), InputTensorShapes.Num());
		return FModelInstanceCoreMLCpu::ERunSyncStatus::Fail;
	}

	if (!InOutputBindings.IsEmpty() && InOutputBindings.Num() != OutputFeatureNames.Num())
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Output binding can be empty or needs to match output feature count (got %d, expected %d).", InOutputBindings.Num(), OutputFeatureNames.Num());
		return FModelInstanceCoreMLCpu::ERunSyncStatus::Fail;
	}
	
	NSMutableDictionary<NSString*, MLFeatureValue*>* InputFeatureValueByNames = [NSMutableDictionary dictionaryWithCapacity:InInputBindings.Num()];
	
	for(int i = 0; i < InputFeatureNames.Num(); ++i)
	{
		NSError* ErrorCreatingInput = nullptr;
		NNE::FTensorShape& InputTensorShape = InputTensorShapes[i];
		NSMutableArray* Shape = [NSMutableArray arrayWithCapacity:InputTensorShape.Rank()];
		NSMutableArray* Strides = [NSMutableArray arrayWithCapacity:InputTensorShape.Rank()];
		MLMultiArrayDataType CoreMLMultiArrayDataType = Detail::GetMultiArrayDataTypeFromType(InputSymbolicTensors[i].GetDataType());
		
		Detail::FillNSArrayShapeAndStrideFromNNEShape(InputTensorShape, Shape, Strides);

		MLMultiArray* InputMultiArray = [[[MLMultiArray alloc]  initWithDataPointer:(void *) InInputBindings[i].Data 
																shape:Shape
																dataType:CoreMLMultiArrayDataType 
																strides:Strides 
																deallocator:^(void * _Nonnull bytes) {}
																error: &ErrorCreatingInput]
															autorelease];
		
		if (ErrorCreatingInput != nullptr)
		{
			FString Error = [ErrorCreatingInput localizedDescription];
			UE_LOGF(LogNNERuntimeCoreML, Error, "Failed to create the input (%ls)", *Error);
			return FModelInstanceCoreMLCpu::ERunSyncStatus::Fail;
		}
		
		[InputFeatureValueByNames setObject:[MLFeatureValue featureValueWithMultiArray:InputMultiArray] forKey:[NSString stringWithFString:InputFeatureNames[i]]];
	}
	
	CoreMLInputWrapper* InputFeatureProvider = [[[CoreMLInputWrapper alloc] init] autorelease];
	InputFeatureProvider.featureByNames = InputFeatureValueByNames;
	
	// Running inference
	NSError* ErrorRunningInference = nullptr;
	MLPredictionOptions* PredictionOptions = [[[MLPredictionOptions alloc] init] autorelease];
	
	id<MLFeatureProvider> OutFeatureProvider = [CoreMLModelInstance predictionFromFeatures:InputFeatureProvider options:PredictionOptions error:&ErrorRunningInference];
	
	if (ErrorRunningInference != nullptr)
	{
		FString Error = [ErrorRunningInference localizedDescription];
		UE_LOGF(LogNNERuntimeCoreML, Error, "Failed to run inference (%ls)", *Error);
		return FModelInstanceCoreMLCpu::ERunSyncStatus::Fail;
	}
	if (!OutFeatureProvider) 
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "No output feature provider was returned");
		return FModelInstanceCoreMLCpu::ERunSyncStatus::Fail;
	}
	
	OutputTensorShapes.Reset();
	for(int i = 0; i < OutputFeatureNames.Num(); ++i)
	{
		NSString* FeatureName = [NSString stringWithFString:OutputFeatureNames[i]];
		MLMultiArray* OutputMultiArray = [OutFeatureProvider featureValueForName:FeatureName].multiArrayValue;
		
		// Capture the shapes of the output feature
		const TArray<uint32, TInlineAllocator<NNE::FTensorShape::MaxRank>> ShapeData = Detail::GetShapeDataFromNSArray<uint32>(OutputMultiArray.shape);
		const NNE::FTensorShape Shape = NNE::FTensorShape::Make(ShapeData);
		
		OutputTensorShapes.Emplace(Shape);
		
		// Copy the memory buffers of the output features to user provided bindings
		if (!InOutputBindings.IsEmpty() && InOutputBindings[i].Data)
		{
			const int32 ElementSize = NNE::GetTensorDataTypeSizeInBytes(Detail::GetTypeFromMultiArrayDataType(OutputMultiArray.dataType));
			
			// OutputMultiArray.count is the product of shape dimensions (logical element count, excludes stride padding)
			const uint64 LogicalSizeInBytes = static_cast<uint64>(OutputMultiArray.count) * ElementSize;

			if (InOutputBindings[i].SizeInBytes >= LogicalSizeInBytes)
			{
				[OutputMultiArray getBytesWithHandler:^(const void* bytes, NSInteger size) {
					FMemory::Memcpy(InOutputBindings[i].Data, bytes, LogicalSizeInBytes);
				}];
			}
		}
	}
	
	return FModelInstanceCoreMLCpu::ERunSyncStatus::Ok;
#else // !defined(__APPLE__)
	return FModelInstanceCoreMLCpu::ERunSyncStatus::Fail;
#endif // defined(__APPLE__)
}

template <class InstanceType>
bool FModelInstanceCoreMLBase<InstanceType>::Init(TConstArrayView64<uint8> ModelData)
{
	SCOPED_NAMED_EVENT_TEXT("FModelInstanceCoreMLBase::Init", FColor::Magenta);

#if defined(__APPLE__)

	SCOPED_AUTORELEASE_POOL;

	using namespace UE::NNERuntimeCoreML;

	FModelData CoreMLModelData;
	if (!CoreMLModelData.Load(ModelData))
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Failed to read model data.");
		return false;
	}

	// Save file data to disk
	FString ProjIntermediateDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir());
	FString TempDirForModel = Detail::GenerateUniqueTempDirPath(ProjIntermediateDir);
	FString Filepath;
	if (CoreMLModelData.ModelType == EModelType::MLModel)
	{
		Filepath = FPaths::Combine(TempDirForModel, TEXT("CoreMLModel.mlmodel"));
		if (!FFileHelper::SaveArrayToFile(CoreMLModelData.FileDataView, *Filepath))
		{
			IFileManager::Get().DeleteDirectory(*TempDirForModel, false, true);
			UE_LOGF(LogNNERuntimeCoreML, Error, "Could not write model to disk at path %ls.", *Filepath);
			return false;
		}
	}
	else if (CoreMLModelData.ModelType == EModelType::MLPackage)
	{
		Filepath = FPaths::Combine(TempDirForModel, TEXT("CoreMLModel.mlpackage"));
		if (FPaths::DirectoryExists(Filepath))
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			if (!PlatformFile.DeleteDirectoryRecursively(*Filepath))
			{
				UE_LOGF(LogNNERuntimeCoreML, Error, "Failed to delete directory %ls", *Filepath);
				return false;
			}
		}

		if (!SaveArrayToDirectory(CoreMLModelData.FileDataView, *Filepath))
		{
			IFileManager::Get().DeleteDirectory(*TempDirForModel, false, true);
			UE_LOGF(LogNNERuntimeCoreML, Error, "Could not write model to disk at path %ls.", *Filepath);
			return false;
		}
	}
	else
	{
		checkf(false, TEXT("Unexpected ModelType %i."), static_cast<int32>(CoreMLModelData.ModelType));
		return false;
	}
	
	// Compile the model with CoreML and cleanup temp directory
	NSURL* ModelURL = [NSURL fileURLWithPath:[NSString stringWithFString:Filepath]];
	NSError* ErrorCompilationObj = nullptr;
	NSURL* CompiledModelURL = [MLModel compileModelAtURL:ModelURL error:&ErrorCompilationObj];
	if (ErrorCompilationObj != nullptr)
	{
		IFileManager::Get().DeleteDirectory(*TempDirForModel, false, true);
		FString Error = [ErrorCompilationObj localizedDescription];
		UE_LOGF(LogNNERuntimeCoreML, Error, "Failed to compile model (%ls)", *Error);
		return false;
	}
	if (!IFileManager::Get().DeleteDirectory(*TempDirForModel, false, true))
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Could not cleanup model temp directory %ls.", *TempDirForModel);
	}

	// Create the model instance from the compiled model.
	NSError* ErrorCreationObj = nullptr;
	MLModelConfiguration* ModelConfiguration = [[[MLModelConfiguration alloc] init] autorelease];
	
	ModelConfiguration.computeUnits = Detail::InstanceTypeTrait<InstanceType>::GetComputeUnits();
	
	MLModel* ModelInstance = [MLModel modelWithContentsOfURL:CompiledModelURL configuration:ModelConfiguration error:&ErrorCreationObj];
	
	if (ErrorCreationObj != nullptr)
	{
		FString Error = [ErrorCreationObj localizedDescription];
		UE_LOGF(LogNNERuntimeCoreML, Error, "Failed to make an instance from compiled model (%ls)", *Error);
		return false;
	}
	
	// Extract model metadata
	if (!Detail::GetTensorDescAndFeatureNamesFromMLDescription(
		InputSymbolicTensors, InputFeatureNames, 
		ModelInstance.modelDescription.inputDescriptionsByName, 
		true, TEXT("inputs")))
	{
		return false;
	}
	if (!Detail::GetTensorDescAndFeatureNamesFromMLDescription(
		OutputSymbolicTensors, OutputFeatureNames, 
		ModelInstance.modelDescription.outputDescriptionsByName, 
		false, TEXT("outputs")))
	{
		return false;
	}

	// Store instance pointer and prevent it to be garbage collected.
	CoreMLModelInstance = ModelInstance;
	
	[CoreMLModelInstance retain];
	
	return true;
#else // !defined(__APPLE__)
	return false;
#endif // defined(__APPLE__)
}

TSharedPtr<NNE::IModelInstanceCPU> FModelCoreMLCpu::CreateModelInstanceCPU()
{
	TSharedPtr<FModelInstanceCoreMLCpu> ModelInstance = MakeShared<FModelInstanceCoreMLCpu>();
	bool IsModelInitialized = ModelInstance->Init(ModelData->GetView());
	
	if (!IsModelInitialized)
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Cannot initialize model instance");
		return TSharedPtr<NNE::IModelInstanceCPU>();
	}

	return ModelInstance;
}

TSharedPtr<NNE::IModelInstanceGPU> FModelCoreMLGpu::CreateModelInstanceGPU()
{
	TSharedPtr<FModelInstanceCoreMLGpu> ModelInstance = MakeShared<FModelInstanceCoreMLGpu>();
	bool IsModelInitialized = ModelInstance->Init(ModelData->GetView());
	
	if (!IsModelInitialized)
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Cannot initialize model instance");
		return TSharedPtr<NNE::IModelInstanceGPU>();
	}

	return ModelInstance;
}

TSharedPtr<NNE::IModelInstanceNPU> FModelCoreMLNpu::CreateModelInstanceNPU()
{
	TSharedPtr<FModelInstanceCoreMLNpu> ModelInstance = MakeShared<FModelInstanceCoreMLNpu>();
	bool IsModelInitialized = ModelInstance->Init(ModelData->GetView());
	
	if (!IsModelInitialized)
	{
		UE_LOGF(LogNNERuntimeCoreML, Error, "Cannot initialize model instance");
		return TSharedPtr<NNE::IModelInstanceNPU>();
	}

	return ModelInstance;
}

FModelCoreMLCpu::FModelCoreMLCpu(TSharedRef<UE::NNE::FSharedModelData> InModelData) :
	ModelData(InModelData)
{
}

FModelCoreMLGpu::FModelCoreMLGpu(TSharedRef<UE::NNE::FSharedModelData> InModelData) :
	ModelData(InModelData)
{
}

FModelCoreMLNpu::FModelCoreMLNpu(TSharedRef<UE::NNE::FSharedModelData> InModelData) :
	ModelData(InModelData)
{
}
	
} // namespace UE::NNERuntimeCoreML::Private

#endif // WITH_NNE_RUNTIME_COREML
