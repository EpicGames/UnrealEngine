// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef WITH_NNE_RUNTIME_COREML

#include "CoreMinimal.h"
#include "NNERuntimeCPU.h"
#include "NNERuntimeGPU.h"
#include "NNERuntimeNPU.h"
#include "NNETypes.h"

#if defined(__APPLE__)
@class MLModel;
#endif

namespace UE::NNE
{
	class FSharedModelData;
}

namespace UE::NNERuntimeCoreML::Private
{

template <class InstanceType>
class FModelInstanceCoreMLBase : public InstanceType
{
public:
	FModelInstanceCoreMLBase() = default;
	virtual ~FModelInstanceCoreMLBase();

	using ESetInputTensorShapesStatus = UE::NNE::IModelInstanceCPU::ESetInputTensorShapesStatus;
	using ERunSyncStatus = UE::NNE::IModelInstanceCPU::ERunSyncStatus;
	
	bool Init(TConstArrayView64<uint8> ModelData);

	virtual TConstArrayView<NNE::FTensorDesc> GetInputTensorDescs() const override;
	virtual TConstArrayView<NNE::FTensorDesc> GetOutputTensorDescs() const override;
	virtual TConstArrayView<NNE::FTensorShape> GetInputTensorShapes() const override;
	virtual TConstArrayView<NNE::FTensorShape> GetOutputTensorShapes() const override;
	virtual ESetInputTensorShapesStatus SetInputTensorShapes(TConstArrayView<NNE::FTensorShape> InInputShapes) override;
	virtual ERunSyncStatus RunSync(TConstArrayView<NNE::FTensorBindingCPU> InInputTensors, TConstArrayView<NNE::FTensorBindingCPU> InOutputTensors) override;

private:
	TArray<NNE::FTensorShape>	InputTensorShapes;
	TArray<NNE::FTensorShape>	OutputTensorShapes;
	TArray<NNE::FTensorDesc>	InputSymbolicTensors;
	TArray<NNE::FTensorDesc>	OutputSymbolicTensors;
#if defined(__APPLE__)
	TArray<FString>				InputFeatureNames;
	TArray<FString>				OutputFeatureNames;
	MLModel *					CoreMLModelInstance;
#endif
};

class FModelInstanceCoreMLCpu : public FModelInstanceCoreMLBase<NNE::IModelInstanceCPU>
{
public:
	FModelInstanceCoreMLCpu() = default;
	virtual ~FModelInstanceCoreMLCpu() = default;
};

class FModelInstanceCoreMLGpu : public FModelInstanceCoreMLBase<NNE::IModelInstanceGPU>
{
public:
	FModelInstanceCoreMLGpu() = default;
	virtual ~FModelInstanceCoreMLGpu() = default;
};

class FModelInstanceCoreMLNpu : public FModelInstanceCoreMLBase<NNE::IModelInstanceNPU>
{
public:
	FModelInstanceCoreMLNpu() = default;
	virtual ~FModelInstanceCoreMLNpu() = default;
};

class FModelCoreMLCpu : public NNE::IModelCPU
{
public:
	FModelCoreMLCpu(TSharedRef<NNE::FSharedModelData> InModelData);
	virtual ~FModelCoreMLCpu() = default;

	virtual TSharedPtr<NNE::IModelInstanceCPU> CreateModelInstanceCPU() override;

private:
	TSharedRef<NNE::FSharedModelData> ModelData;
};

class FModelCoreMLGpu : public NNE::IModelGPU
{
public:
	FModelCoreMLGpu(TSharedRef<NNE::FSharedModelData> InModelData);
	virtual ~FModelCoreMLGpu() = default;

	virtual TSharedPtr<NNE::IModelInstanceGPU> CreateModelInstanceGPU() override;

private:
	TSharedRef<NNE::FSharedModelData> ModelData;
};

class FModelCoreMLNpu : public NNE::IModelNPU
{
public:
	FModelCoreMLNpu(TSharedRef<NNE::FSharedModelData> InModelData);
	virtual ~FModelCoreMLNpu() = default;

	virtual TSharedPtr<NNE::IModelInstanceNPU> CreateModelInstanceNPU() override;

private:
	TSharedRef<NNE::FSharedModelData> ModelData;
};
	
} // UE::NNERuntimeCoreML::Private

#endif // WITH_NNE_RUNTIME_COREML
