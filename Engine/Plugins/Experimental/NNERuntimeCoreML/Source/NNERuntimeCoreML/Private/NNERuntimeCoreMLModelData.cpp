// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeCoreMLModelData.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

namespace UE::NNERuntimeCoreML
{
	bool FModelData::Load(TConstArrayView64<uint8> InData)
	{
		FMemoryReaderView Reader(InData, /*bIsPersistent*/true);
		Reader << GUID;
		Reader << Version;
		Reader << ModelType;

		const int64 Offset = Reader.Tell();
		if (Reader.IsError() || Offset > InData.Num())
		{
			return false;
		}

		FileDataView = InData.Slice(Offset, InData.Num() - Offset);
		
		return true;
	}

	bool FModelData::Store(TArray64<uint8>& OutData)
	{
		FMemoryWriter64 Writer(OutData, /*bIsPersistent*/true);
		Writer << GUID;
		Writer << Version;
		Writer << ModelType;
		Writer.Serialize((void*)FileDataView.GetData(), FileDataView.Num());
		
		return !Writer.IsError();
	}

	bool FModelData::CheckGUIDAndVersion(const FGuid& ReferenceGUID, int32 ReferenceVersion) const
	{
		return ReferenceGUID == GUID && ReferenceVersion == Version;
	}
}