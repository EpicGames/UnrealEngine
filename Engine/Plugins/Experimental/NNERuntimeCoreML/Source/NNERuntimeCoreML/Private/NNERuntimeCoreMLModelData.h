// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Misc/Guid.h"

namespace UE::NNERuntimeCoreML
{
	enum class EModelType : uint8
	{
		MLModel,
		MLPackage
	};

	class FModelData
	{
	public:
		FGuid GUID;

		int32 Version = 0;

		EModelType ModelType = EModelType::MLModel;
		
		// Non-owning view into file data. On saving, copy of file data is written. When loading, points to source passed to Load().
		TConstArrayView64<uint8> FileDataView;

		bool Load(TConstArrayView64<uint8> InData);

		bool Store(TArray64<uint8>& OutData);

		bool CheckGUIDAndVersion(const FGuid& ReferenceGUID, int32 ReferenceVersion) const;
	};
}