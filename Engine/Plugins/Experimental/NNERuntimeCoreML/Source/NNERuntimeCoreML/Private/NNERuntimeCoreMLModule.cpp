// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeCoreMLModule.h"
#include "NNE.h"
#include "NNERuntimeCoreML.h"
#include "NNERuntimeCoreMLLog.h"
#include "NNERuntimeCoreMLNPUHelper.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#if defined(__APPLE__)
#import <Foundation/Foundation.h>
#import <CoreML/CoreML.h>

#include "Apple/ScopeAutoreleasePool.h"

#endif // defined(__APPLE__)

DEFINE_LOG_CATEGORY(LogNNERuntimeCoreML);

namespace UE::NNERuntimeCoreML::Private::Details
{

bool IsInferenceSupported()
{
#if defined(__APPLE__)
	return true;
#else // !defined(__APPLE__)
	return false;
#endif // defined(__APPLE__)
}

bool IsNPUAvailable()
{
#if defined(__APPLE__)
	SCOPED_AUTORELEASE_POOL;
	
	NSArray<id<MLComputeDeviceProtocol>>* Devices = MLAllComputeDevices();
	
	for(id<MLComputeDeviceProtocol> Device in Devices)
	{
		if ([Device isKindOfClass:[MLNeuralEngineComputeDevice class]]) {
			return true;
		}
	}
	return false;
#else // !defined(__APPLE__)
	return false;
#endif // defined(__APPLE__)
}

} // namespace namespace UE::NNERuntimeCoreML::Private::Details

void FNNERuntimeCoreMLModule::StartupModule()
{
#ifdef WITH_NNE_RUNTIME_COREML
	if (UE::NNERuntimeCoreML::Private::Details::IsInferenceSupported())
	{
		if (UE::NNERuntimeCoreML::Private::Details::IsNPUAvailable())
		{
			NNERuntimeCoreML = NewObject<UNNERuntimeCoreMLCpuGpuNpu>();
		}
		else
		{
			NNERuntimeCoreML = NewObject<UNNERuntimeCoreMLCpuGpu>();
		}
	} 
	else
	{
		NNERuntimeCoreML = NewObject<UNNERuntimeCoreML>();
	}
	
	if (NNERuntimeCoreML.IsValid())
	{
		TWeakInterfacePtr<INNERuntime> RuntimeInterface(NNERuntimeCoreML.Get());
		
		NNERuntimeCoreML->AddToRoot();
		UE::NNE::RegisterRuntime(RuntimeInterface);
	}
#endif // WITH_NNE_RUNTIME_COREML
}

void FNNERuntimeCoreMLModule::ShutdownModule()
{
#ifdef WITH_NNE_RUNTIME_COREML
	if (NNERuntimeCoreML.IsValid())
	{
		TWeakInterfacePtr<INNERuntime> RuntimeInterface(NNERuntimeCoreML.Get());
		
		UE::NNE::UnregisterRuntime(RuntimeInterface);
		NNERuntimeCoreML->RemoveFromRoot();
		NNERuntimeCoreML.Reset();
	}
#endif // WITH_NNE_RUNTIME_COREML
}

IMPLEMENT_MODULE(FNNERuntimeCoreMLModule, NNERuntimeCoreML);
