// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

#ifdef WITH_NNE_RUNTIME_COREML
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UNNERuntimeCoreML;
#endif // WITH_NNE_RUNTIME_COREML

class FNNERuntimeCoreMLModule : public IModuleInterface
{
private:
#ifdef WITH_NNE_RUNTIME_COREML
	TWeakObjectPtr<UNNERuntimeCoreML> NNERuntimeCoreML{ nullptr };
#endif // WITH_NNE_RUNTIME_COREML
	
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
