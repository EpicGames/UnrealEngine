// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeCoreMLNPUHelper.h"
#include "HAL/Platform.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#if defined(__APPLE__)
#include "Apple/ScopeAutoreleasePool.h"
#include "CoreMinimal.h"

#import <Foundation/Foundation.h>
#import <CoreML/CoreML.h>
#endif // defined(__APPLE__)

namespace UE::NNERuntimeCoreML::Private
{

bool IsNPUAvailable()
{
#if defined(__APPLE__)
	SCOPED_AUTORELEASE_POOL;
	
	NSArray<id<MLComputeDeviceProtocol>>* Devices = MLAllComputeDevices();
	
	for(id<MLComputeDeviceProtocol> Device in Devices)
	{
		if ([Device isKindOfClass:[MLNeuralEngineComputeDevice class]]) {
			return true;
		}
	}
	return false;
#else // !defined(__APPLE__)
	return false;
#endif // defined(__APPLE__)
}

} // namespace UE::NNERuntimeCoreML::Private
