// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::NNERuntimeCoreML::Private
{

bool IsNPUAvailable();

} // namespace UE::NNERuntimeCoreML::Private
