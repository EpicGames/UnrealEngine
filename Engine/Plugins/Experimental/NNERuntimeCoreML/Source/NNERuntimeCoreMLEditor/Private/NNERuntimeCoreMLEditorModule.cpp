// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeCoreMLEditorModule.h"
#include "NNERuntimeCoreMLLog.h"

#define LOCTEXT_NAMESPACE "FNNERuntimeCoreMLEditorModule"

void FNNERuntimeCoreMLEditorModule::StartupModule()
{
}

void FNNERuntimeCoreMLEditorModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FNNERuntimeCoreMLEditorModule, NNERuntimeCoreMLEditor)
