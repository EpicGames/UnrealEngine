// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeCoreMLModelDataFactory.h"

#include "Editor.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "NNEModelData.h"
#include "NNERuntimeCoreMLLog.h"
#include "NNERuntimeCoreMLUtils.h"
#include "Subsystems/ImportSubsystem.h"

UNNERuntimeCoreMLModelDataFactory::UNNERuntimeCoreMLModelDataFactory(const FObjectInitializer& ObjectInitializer) : UFactory(ObjectInitializer)
{
	bCreateNew = false;
	bEditorImport = true;
	SupportedClass = UNNEModelData::StaticClass();
	ImportPriority = DefaultImportPriority + 100;
	Formats.Add(TEXT("mlmodel;Core ML model format"));
	Formats.Add(TEXT("json;Core ML ML Program format"));
}

UObject* UNNERuntimeCoreMLModelDataFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	SCOPED_NAMED_EVENT_TEXT("UNNERuntimeCoreMLModelDataFactory::FactoryCreateFile", FColor::Magenta);
	
	using namespace UE::NNERuntimeCoreML;

	const FString Extension = FPaths::GetExtension(Filename);
	check(Extension == TEXT("mlmodel") || Extension == TEXT("json"));

	const bool bIsMLModel = Extension == TEXT("mlmodel");
	const FString Type = bIsMLModel ? TEXT("mlmodel") : TEXT("mlpackage");

	// Determine the correct asset name
	FString AssetName = InName.ToString();
	UObject* Parent = InParent;

	const FString ParentPath = FPaths::GetPath(Filename);

	if (!bIsMLModel)
	{
		// Extract name from "ModelName.mlpackage"
		AssetName = FPaths::GetBaseFilename(ParentPath);

		// Create new package with correct name
		const FString PackageName = FPackageName::GetLongPackagePath(InParent->GetName());
		const FString NewPackageName = PackageName / AssetName;
		
		UPackage* Pkg = CreatePackage(*NewPackageName);
		if(!ensure(Pkg))
		{
			UE_LOGF(LogNNERuntimeCoreML, Error, "Failed to create package '%ls' for asset '%ls'", *NewPackageName, *AssetName);
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			return nullptr;
		}

		Pkg->FullyLoad();
		Parent = Pkg;

		UObject* ExistingObject = StaticFindObject(UObject::StaticClass(), Pkg, *AssetName);
		if (ExistingObject)
		{
			// Overwrite existing asset if it's the same type, otherwise cancel import to avoid data loss
			if (DoesSupportClass(ExistingObject->GetClass()))
			{
				UE_LOGF(LogNNERuntimeCoreML, Log, "Replaced existing Core ML asset '%ls' in package '%ls'.", *AssetName, *NewPackageName);
			}
			else
			{
				UE_LOGF(LogNNERuntimeCoreML, Warning, "An asset named '%ls' already exists in package '%ls', will not import.", *AssetName, *NewPackageName);

				bOutOperationCanceled = true;

				GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
				return nullptr;
			}
		}
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, Parent, FName(*AssetName), *Type);
	
	TArray64<uint8> Data;
	if (bIsMLModel)
	{
		if (!FFileHelper::LoadFileToArray(Data, *Filename))
		{
			UE_LOGF(LogNNERuntimeCoreML, Error, "Failed to load file '%ls' to array", *Filename);
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			return nullptr;
		}
	}
	else
	{
		if (!LoadDirectoryToArray(Data, *ParentPath))
		{
			UE_LOGF(LogNNERuntimeCoreML, Error, "Failed to load folder '%ls' to array", *ParentPath);
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			return nullptr;
		}
	}

	UNNEModelData* ModelData = NewObject<UNNEModelData>(Parent, InClass, FName(*AssetName), Flags);
	check(ModelData)

	ModelData->Init(Type, Data);

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, ModelData);
	return ModelData;
}

bool UNNERuntimeCoreMLModelDataFactory::FactoryCanImport(const FString & Filename)
{
	if (Filename.EndsWith(TEXT(".mlmodel")))
	{
		return true;
	}
	
	const FString CleanFilename = FPaths::GetCleanFilename(Filename);
	const FString ParentPath = FPaths::GetPath(Filename);
	if (CleanFilename == TEXT("Manifest.json") && ParentPath.EndsWith(TEXT(".mlpackage")))
	{
		return true;
	}

	return false;
}
