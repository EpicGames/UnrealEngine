// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"

#include "NNERuntimeCoreMLModelDataFactory.generated.h"

UCLASS()
class NNERUNTIMECOREMLEDITOR_API UNNERuntimeCoreMLModelDataFactory : public UFactory
{
	GENERATED_BODY()

public:
	UNNERuntimeCoreMLModelDataFactory(const FObjectInitializer& ObjectInitializer);

public:
	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	//~ End UFactory Interface
};
