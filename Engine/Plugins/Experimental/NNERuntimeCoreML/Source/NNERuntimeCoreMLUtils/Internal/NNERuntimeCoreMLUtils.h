// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"

namespace UE::NNERuntimeCoreML
{
	NNERUNTIMECOREMLUTILS_API bool LoadDirectoryToArray(TArray64<uint8>& Result, const FString& Path);

	NNERUNTIMECOREMLUTILS_API bool SaveArrayToDirectory(TConstArrayView64<uint8> Data, const TCHAR* Path);
};