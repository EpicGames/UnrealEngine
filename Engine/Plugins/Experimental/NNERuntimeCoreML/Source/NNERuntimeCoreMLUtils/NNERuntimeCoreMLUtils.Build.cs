// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NNERuntimeCoreMLUtils : ModuleRules
{
	public NNERuntimeCoreMLUtils(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core"
			}
		);
	}
}
