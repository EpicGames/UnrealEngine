// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNERuntimeCoreMLUtils.h"

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Math/Color.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "NNERuntimeCoreMLUtilsLog.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/UnrealTemplate.h"

namespace UE::NNERuntimeCoreML
{

namespace Private
{

constexpr int32 FolderSerializationVersion = 1;

struct FSerializedFileEntry
{
	FString RelativePath;
	int64 FileSize;

	friend FArchive& operator<<(FArchive& Ar, FSerializedFileEntry& Entry)
	{
		Ar << Entry.RelativePath;
		Ar << Entry.FileSize;
		return Ar;
	}
};

struct FSerializedDirectoryHeader
{
	int32 Version = FolderSerializationVersion;
	TArray<FSerializedFileEntry> Files;

	friend FArchive& operator<<(FArchive& Ar, FSerializedDirectoryHeader& Header)
	{
		Ar << Header.Version;
		Ar << Header.Files;
		return Ar;
	}
};

struct FFileInfo
{
	FString FullPath;
	FSerializedFileEntry FileEntry;
};

} // namespace Private

bool LoadDirectoryToArray(TArray64<uint8>& Result, const FString& Path)
{
	SCOPED_NAMED_EVENT_TEXT("CoreML::LoadDirectoryToArray", FColor::Magenta);
	
	using namespace Private;

	TArray<FFileInfo> Files;
	int64 TotalContentSize = 0;

	FString DirectoryPath = Path;
	FPaths::NormalizeDirectoryName(DirectoryPath);
	if (!DirectoryPath.EndsWith(TEXT("/")))
	{
		DirectoryPath.Append(TEXT("/"));
	}

	const bool bSuccess = IFileManager::Get().IterateDirectoryStatRecursively(*DirectoryPath,
		[&DirectoryPath, &Files, &TotalContentSize](const TCHAR* FilenameOrDirectory, const FFileStatData& StatData)
		{
			if (!StatData.bIsDirectory)
			{
				if (StatData.FileSize < 0)
				{
					UE_LOGF(LogNNERuntimeCoreMLUtils, Warning, "File %ls has unknown size (FileSize is %lld)", FilenameOrDirectory, StatData.FileSize);
					return false;
				}

				FString RelativePath(FilenameOrDirectory);
				if (!FPaths::MakePathRelativeTo(RelativePath, *DirectoryPath))
				{
					UE_LOGF(LogNNERuntimeCoreMLUtils, Warning, "Failed to convert %ls to a relative path of %ls", FilenameOrDirectory, *DirectoryPath);
					return false;
				}

				TotalContentSize += StatData.FileSize;

				Files.Add(
				{
					.FullPath = FString(FilenameOrDirectory),
					.FileEntry = { .RelativePath = MoveTemp(RelativePath), .FileSize = StatData.FileSize }
				});
			}
			return true;
		});

	if (!bSuccess)
	{
		return false;
	}

	FSerializedDirectoryHeader Header;
	Header.Version = FolderSerializationVersion;
	Header.Files.Reserve(Files.Num());
	for (const FFileInfo& Info : Files)
	{
		Header.Files.Add(Info.FileEntry);
	}

	TArray<uint8> HeaderBuffer;
	FMemoryWriter HeaderWriter(HeaderBuffer, /*bIsPersistent*/true);
	HeaderWriter << Header;

	const int64 TotalSize = HeaderBuffer.Num() + TotalContentSize;
	Result.SetNumUninitialized(TotalSize);

	// Copy header directly
	FMemory::Memcpy(Result.GetData(), HeaderBuffer.GetData(), HeaderBuffer.Num());

	int64 WriteOffset = HeaderBuffer.Num();

	for (const FFileInfo& Info : Files)
	{
		SCOPED_NAMED_EVENT_TEXT("CoreML::LoadDirectoryToArray::Loop_file_read", FColor::Magenta);
		TUniquePtr<IFileHandle> FileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenRead(*Info.FullPath));
		if (!FileHandle || !FileHandle->Read(Result.GetData() + WriteOffset, Info.FileEntry.FileSize))
		{
			UE_LOGF(LogNNERuntimeCoreMLUtils, Warning, "Failed to read from %ls", *Info.FullPath);
			return false;
		}
		WriteOffset += Info.FileEntry.FileSize;
	}
 
	ensureMsgf(Result.Num() == TotalSize, TEXT("Serialization used different amount than estimated: %lld (estimated %lld)."), Result.Num(), TotalSize);
	
	return !HeaderWriter.IsError();
}

bool SaveArrayToDirectory(TConstArrayView64<uint8> Data, const TCHAR* Path)
{
	SCOPED_NAMED_EVENT_TEXT("CoreML::SaveArrayToDirectory", FColor::Magenta);

	using namespace Private;

	FString BasePath = Path;
	FPaths::NormalizeDirectoryName(BasePath);

	if (!IFileManager::Get().MakeDirectory(*BasePath, /*Tree=*/true))
	{
		UE_LOGF(LogNNERuntimeCoreMLUtils, Warning, "Failed to create directory %ls", *BasePath);
		return false;
	}

	FMemoryReaderView Reader(Data);
	FSerializedDirectoryHeader Header;
	Reader << Header;

	if (Header.Version != FolderSerializationVersion)
	{
		UE_LOGF(LogNNERuntimeCoreMLUtils, Warning, "Unexpected version %i encountered", Header.Version);
		return false;
	}

	FString BasePathForCheck = BasePath;
	if (!BasePathForCheck.EndsWith(TEXT("/")))
	{
		BasePathForCheck.Append(TEXT("/"));
	}

	for (const FSerializedFileEntry& Entry : Header.Files)
	{
		const FString AbsolutePath = FPaths::Combine(Path, Entry.RelativePath);
		FString NormalizedPath = AbsolutePath;
		FPaths::CollapseRelativeDirectories(NormalizedPath);

		if (!NormalizedPath.StartsWith(BasePathForCheck))
		{
			UE_LOGF(LogNNERuntimeCoreMLUtils, Warning, "Path traversal detected: %ls resolved outside base directory", *Entry.RelativePath);
			return false;
		}
		
		int64 Start = Reader.Tell();
		int64 End = Start + Entry.FileSize;
		if (Entry.FileSize < 0 || End > Data.Num())
		{
			UE_LOGF(LogNNERuntimeCoreMLUtils, Warning, "Corrupted File Size. Read %lld at location %lld but data only has total size %lld.", Entry.FileSize, Start, Data.Num());
			return false;
		}

		TConstArrayView64<uint8> Buffer(Data.GetData() + Start, Entry.FileSize);
		if (!FFileHelper::SaveArrayToFile(Buffer, *NormalizedPath))
		{
			UE_LOGF(LogNNERuntimeCoreMLUtils, Warning, "Save to file %ls failed", *NormalizedPath);
			return false;
		}

		Reader.Seek(End);
	}

	return !Reader.IsError();
}

} // namespace UE::NNERuntimeCoreML