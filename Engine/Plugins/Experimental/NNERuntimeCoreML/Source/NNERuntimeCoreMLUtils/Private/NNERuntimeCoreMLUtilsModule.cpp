// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "NNERuntimeCoreMLUtilsLog.h"

DEFINE_LOG_CATEGORY(LogNNERuntimeCoreMLUtils);

class FNNERuntimeCoreMLUtilsModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override {}
	virtual void ShutdownModule() override {}
};

IMPLEMENT_MODULE(FNNERuntimeCoreMLUtilsModule, NNERuntimeCoreMLUtils)